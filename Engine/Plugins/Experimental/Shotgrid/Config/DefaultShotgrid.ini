[CoreRedirects]
+ClassRedirects=(OldName="ShotgunEngine", NewName="/Script/Shotgrid.ShotgridEngine")
+ClassRedirects=(OldName="ShotgunSettings", NewName="/Script/Shotgrid.ShotgridSettings")
+PackageRedirects=(OldName="/Script/Shotgun",NewName="/Script/Shotgrid")

