{
	"FileVersion" : 1,
	"Version" : 1,
	"VersionName" : "0.10",
	"FriendlyName" : "Flow Production Tracking",
	"Description" : "Flow Production Tracking (formerly known as Shotgun and/or ShotGrid) integration for the Unreal Editor.",
	"Category" : "Editor",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : true,
	"IsBetaVersion" : true,
	"Installed" : false,
	"Modules":
	[
		{
			"Name": "Shotgrid",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList":
			[
				"Win64",
				"Linux"
			]
		}
	],
	"Plugins":
	[
		{
			"Name": "PythonScriptPlugin",
			"Enabled": true
		},
		{
			"Name": "EditorScriptingUtilities",
			"Enabled": true
		}
	]
}
