// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShotgridEngine.h"

#include "AssetRegistry/AssetRegistryModule.h"

#include "GameFramework/Actor.h"
#include "IPythonScriptPlugin.h"
#include "Misc/Paths.h"

UShotgridEngine* UShotgridEngine::GetInstance()
{
	// The Python Shotgrid Engine instance must come from a Python class derived from UShotgridEngine
	// In Python, there should be only one derivation, but hot-reloading will create new derived classes, so use the last one
	TArray<UClass*> ShotgridEngineClasses;
	GetDerivedClasses(UShotgridEngine::StaticClass(), ShotgridEngineClasses);
	int32 NumClasses = ShotgridEngineClasses.Num();
	if (NumClasses > 0)
	{
		return Cast<UShotgridEngine>(ShotgridEngineClasses[NumClasses - 1]->GetDefaultObject());
	}
	return nullptr;
}

static void OnEditorExit()
{
	if (UShotgridEngine* Engine = UShotgridEngine::GetInstance())
	{
		Engine->Shutdown();
	}
}

void UShotgridEngine::OnEngineInitialized() const
{
	IPythonScriptPlugin::Get()->OnPythonShutdown().AddStatic(OnEditorExit);
}

void UShotgridEngine::SetSelection(const TArray<FAssetData>* InSelectedAssets, const TArray<AActor*>* InSelectedActors)
{
	SelectedAssets.Reset();
	WeakSelectedActors.Reset();

	if (InSelectedAssets)
	{
		SelectedAssets = *InSelectedAssets;
	}
	if (InSelectedActors)
	{
		// Also set the assets referenced by the selected actors as selected assets
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
		TArray<FAssetData> AllReferencedAssets;
		for (const AActor* Actor : *InSelectedActors)
		{
			WeakSelectedActors.Add(Actor);

			TArray<UObject*> ActorAssets = GetReferencedAssets(Actor);
			for (UObject* Asset : ActorAssets)
			{
				if (IsValid(Asset) && Asset->IsAsset())
				{
					FAssetData AssetData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(Asset));
					AllReferencedAssets.AddUnique(MoveTemp(AssetData));
				}
			}
		}
		SelectedAssets = MoveTemp(AllReferencedAssets);
	}
}

TArray<AActor*> UShotgridEngine::GetSelectedActors()
{
	TArray<AActor*> Actors;
	for (const FWeakObjectPtr& ObjPtr : WeakSelectedActors)
	{
		if (AActor* Actor = Cast<AActor>(ObjPtr.Get()))
		{
			Actors.Add(Actor);
		}
	}
	return Actors;
}

TArray<UObject*> UShotgridEngine::GetReferencedAssets(const AActor* Actor) const
{
	TArray<UObject*> ReferencedAssets;

	if (Actor)
	{
		Actor->GetReferencedContentObjects(ReferencedAssets);
	}

	return ReferencedAssets;
}

FString UShotgridEngine::GetShotgridWorkDir()
{
	return FPaths::ConvertRelativePathToFull(FPaths::ProjectDir());
}
