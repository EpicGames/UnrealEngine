// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"

#include "ShotgridEngine.generated.h"

class AActor;

USTRUCT(Blueprintable)
struct FShotgridMenuItem
{
	GENERATED_BODY()

public:
	/** Command name for internal use */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Python)
	FString Name;

	/** Text to display in the menu */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Python)
	FString Title;

	/** Description text for the tooltip */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Python)
	FString Description;

	/** Menu item type to help interpret the command */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Python)
	FString Type;
};


/**
 * Wrapper for the Python Shotgrid Engine
 * The functions are implemented in Python by a class that derives from this one
 */
UCLASS(Blueprintable)
class UShotgridEngine : public UObject
{
	GENERATED_BODY()

public:
	/** Get the instance of the Python Shotgrid Engine */
	UFUNCTION(BlueprintCallable, Category = Python)
	static UShotgridEngine* GetInstance();

	/** Callback for when the Python Shotgrid Engine has finished initialization */
	UFUNCTION(BlueprintCallable, Category = Python)
	void OnEngineInitialized() const;

	/** Get the available Shotgrid commands from the Python Shotgrid Engine */
	UFUNCTION(BlueprintImplementableEvent, Category = Python)
	TArray<FShotgridMenuItem> GetShotgridMenuItems() const;

	/** Execute a Shotgrid command by name in the Python Shotgrid Engine */
	UFUNCTION(BlueprintImplementableEvent, Category = Python)
	void ExecuteCommand(const FString& CommandName) const;

	/** Shut down the Python Shotgrid Engine */
	UFUNCTION(BlueprintImplementableEvent, Category = Python)
	void Shutdown() const;

	/** Set the selected objects that will be used to determine the Shotgrid Engine context and execute Shotgrid commands */
	void SetSelection(const TArray<FAssetData>* InSelectedAssets, const TArray<AActor*>* InSelectedActors);

	/** Get the assets that are referenced by the given Actor */
	UFUNCTION(BlueprintCallable, Category = Python)
	TArray<UObject*> GetReferencedAssets(const AActor* Actor) const;

	/** Get the root path for the Shotgrid work area */
	UFUNCTION(BlueprintCallable, Category = Python)
	static FString GetShotgridWorkDir();

	/** Selected assets to be used for Shotgrid commands */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Python)
	TArray<FAssetData> SelectedAssets;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Python, meta = (Deprecated, DeprecationMessage = "SelectedActors is deprecated, use GetSelectedActors instead." ))
	TArray<TObjectPtr<AActor>> SelectedActors;

	/** Selected actors to be used for Shotgrid commands */
	UFUNCTION(BlueprintCallable, Category = Python)
	TArray<AActor*> GetSelectedActors();

	TArray<FWeakObjectPtr> WeakSelectedActors;
};
