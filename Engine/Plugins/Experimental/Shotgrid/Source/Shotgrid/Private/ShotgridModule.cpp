// Copyright Epic Games, Inc. All Rights Reserved.

#include "IShotgridModule.h"
#include "ShotgridSettings.h"
#include "ShotgridUIManager.h"
#include "Misc/CoreDelegates.h"
#include "ISettingsModule.h"
#include "ISettingsSection.h"

#define LOCTEXT_NAMESPACE "Shotgrid"

class FShotgridModule : public IShotgridModule
{
public:

	static void OnEngineStartupComplete()
	{
		RegisterSettings();

		FShotgridUIManager::Initialize();
	}

	static void OnEnginePreExit()
	{
		FShotgridUIManager::Shutdown();

		UnregisterSettings();
	}
	virtual void StartupModule() override
	{
		if (GIsEditor && !IsRunningCommandlet())
		{
			FCoreDelegates::OnFEngineLoopInitComplete.AddStatic(&OnEngineStartupComplete);
			FCoreDelegates::OnEnginePreExit.AddStatic(&OnEnginePreExit);
		}
	}

	virtual void ShutdownModule() override
	{
	}

protected:
	static void RegisterSettings()
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
		if (SettingsModule != nullptr)
		{
			ISettingsSectionPtr SettingsSection = SettingsModule->RegisterSettings("Project", "Plugins", "Shotgrid",
				LOCTEXT("ShotgridSettingsName", "ShotGrid"),
				LOCTEXT("ShotgridSettingsDescription", "Configure the ShotGrid plugin."),
				GetMutableDefault<UShotgridSettings>()
			);
		}
	}

	static void UnregisterSettings()
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
		if (SettingsModule != nullptr)
		{
			SettingsModule->UnregisterSettings("Project", "Plugins", "Shotgrid");
		}
	}
};

IMPLEMENT_MODULE(FShotgridModule, Shotgrid);

#undef LOCTEXT_NAMESPACE
