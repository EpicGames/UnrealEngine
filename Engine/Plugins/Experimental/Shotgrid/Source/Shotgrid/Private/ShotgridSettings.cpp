// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShotgridSettings.h"

#if WITH_EDITOR

DEFINE_LOG_CATEGORY_STATIC(LogShotgrid, Log, All);

void UShotgridSettings::PostInitProperties()
{
	Super::PostInitProperties();

	ApplyMetaDataTagsSettings();
}

void UShotgridSettings::ApplyMetaDataTagsSettings()
{
	TSet<FName>& GlobalTagsForAssetRegistry = UObject::GetMetaDataTagsForAssetRegistry();
	for (FName Tag : MetaDataTagsForAssetRegistry)
	{
		if (!Tag.IsNone())
		{
			if (!GlobalTagsForAssetRegistry.Contains(Tag))
			{
				GlobalTagsForAssetRegistry.Add(Tag);
			}
			else
			{
				// To catch the case where the same tag is used by different users and their settings are synced after edition
				UE_LOGF(LogShotgrid, Warning, "Cannot use duplicate metadata tag '%ls' for Asset Registry", *Tag.ToString());
			}
		}
	}
}

void UShotgridSettings::ClearMetaDataTagsSettings()
{
	TSet<FName>& GlobalTagsForAssetRegistry = UObject::GetMetaDataTagsForAssetRegistry();
	for (FName Tag : MetaDataTagsForAssetRegistry)
	{
		if (!Tag.IsNone())
		{
			GlobalTagsForAssetRegistry.Remove(Tag);
		}
	}
}

void UShotgridSettings::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	if (PropertyAboutToChange != nullptr)
	{
		FName PropertyName = PropertyAboutToChange->GetFName();
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UShotgridSettings, MetaDataTagsForAssetRegistry))
		{
			ClearMetaDataTagsSettings();
		}
	}
}

void UShotgridSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FName PropertyName = (PropertyChangedEvent.MemberProperty != NULL) ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UShotgridSettings, MetaDataTagsForAssetRegistry))
	{
		if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet)
		{
			// Check if the new value already exists in the global tags list
			int32 Index = PropertyChangedEvent.GetArrayIndex(PropertyName.ToString());
			if (Index > 0)
			{
				TSet<FName>::TIterator It = MetaDataTagsForAssetRegistry.CreateIterator();
				for (int32 i = 0; i < Index; ++i)
				{
					++It;
				}
				FName NewValue = It ? *It : FName();
				if (UObject::GetMetaDataTagsForAssetRegistry().Contains(NewValue))
				{
					*It = FName();
					UE_LOGF(LogShotgrid, Warning, "Cannot use duplicate metadata tag '%ls' for Asset Registry", *NewValue.ToString());
				}
			}
		}
		ApplyMetaDataTagsSettings();
	}
}
#endif