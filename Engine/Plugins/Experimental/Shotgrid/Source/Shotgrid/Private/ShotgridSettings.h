// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "ShotgridSettings.generated.h"

struct FPropertyChangedEvent;

UCLASS(config = Game)
class UShotgridSettings : public UObject
{
	GENERATED_BODY()

public:
	/** The metadata tags to be transferred to the Asset Registry. */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = "Asset Registry", DisplayName = "Metadata Tags For Asset Registry")
	TSet<FName> MetaDataTagsForAssetRegistry;

#if WITH_EDITOR
	virtual void PostInitProperties() override;
	virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

private:
	void ApplyMetaDataTagsSettings();
	void ClearMetaDataTagsSettings();
#endif
};
