// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShotgridStyle.h"

#include "Brushes/SlateImageBrush.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyle.h"
#include "Rendering/SlateRenderer.h"
#include "Styling/SlateStyleRegistry.h"

TUniquePtr< FSlateStyleSet > FShotgridStyle::ShotgridStyleInstance = nullptr;

void FShotgridStyle::Initialize()
{
	if (!ShotgridStyleInstance.IsValid())
	{
		ShotgridStyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*ShotgridStyleInstance);
	}
}

void FShotgridStyle::Shutdown()
{
	if (ShotgridStyleInstance.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*ShotgridStyleInstance);
		ShotgridStyleInstance.Reset();
	}
}

FName FShotgridStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("ShotgridStyle"));
	return StyleSetName;
}

FName FShotgridStyle::GetContextName()
{
	static FName ContextName(TEXT("Shotgrid"));
	return ContextName;
}

#define IMAGE_BRUSH( RelativePath, ... ) FSlateImageBrush( Style->RootToContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )

const FVector2D Icon20x20(20.0f, 20.0f);
const FVector2D Icon40x40(40.0f, 40.0f);

TUniquePtr< FSlateStyleSet > FShotgridStyle::Create()
{
	TUniquePtr< FSlateStyleSet > Style = MakeUnique<FSlateStyleSet>(GetStyleSetName());
	Style->SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Experimental/Shotgrid/Resources"));

	return Style;
}

void FShotgridStyle::SetIcon(const FString& StyleName, const FString& ResourcePath)
{
	FSlateStyleSet* Style = ShotgridStyleInstance.Get();

	FString Name(GetContextName().ToString());
	Name = Name + "." + StyleName;
	Style->Set(*Name, new IMAGE_BRUSH(ResourcePath, Icon40x40));

	Name += ".Small";
	Style->Set(*Name, new IMAGE_BRUSH(ResourcePath, Icon20x20));

	FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
}

#undef IMAGE_BRUSH

const ISlateStyle& FShotgridStyle::Get()
{
	check(ShotgridStyleInstance);
	return *ShotgridStyleInstance;
}
