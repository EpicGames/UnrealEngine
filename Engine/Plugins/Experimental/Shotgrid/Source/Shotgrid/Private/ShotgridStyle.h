// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UniquePtr.h"

class FSlateStyleSet;
class ISlateStyle;

/** Manages the style that provides resources for Shotgrid integration */
class FShotgridStyle
{
public:
	static void Initialize();
	static void Shutdown();
	static const ISlateStyle& Get();
	static FName GetStyleSetName();
	static FName GetContextName();

	/** Adds an icon (.png in the given ResourcePath) with the given style name to this style set */
	static void SetIcon(const FString& StyleName, const FString& ResourcePath);

private:
	static TUniquePtr< FSlateStyleSet > Create();
	static TUniquePtr< FSlateStyleSet > ShotgridStyleInstance;
};
