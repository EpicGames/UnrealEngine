// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShotgridUIManager.h"
#include "ShotgridEngine.h"
#include "ShotgridStyle.h"

#include "ToolMenu.h"
#include "ToolMenus.h"

#include "ContentBrowserModule.h"
#include "LevelEditor.h"
#include "Misc/Paths.h"

#include "IPythonScriptPlugin.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"

#define LOCTEXT_NAMESPACE "Shotgrid"

#define LEVELEDITOR_MODULE_NAME TEXT("LevelEditor")
#define CONTENTBROWSER_MODULE_NAME TEXT("ContentBrowser")

namespace UE::Private::Shotgrid
{

void GenerateShotgridMenuContent(FMenuBuilder& MenuBuilder, const TArray<FAssetData>* SelectedAssets, const TArray< AActor*>* SelectedActors)
{
	if (UShotgridEngine* Engine = UShotgridEngine::GetInstance())
	{
		Engine->SetSelection(SelectedAssets, SelectedActors);

		// Query the available Shotgrid commands from the Shotgrid engine
		TArray<FShotgridMenuItem> MenuItems = Engine->GetShotgridMenuItems();
		for (const FShotgridMenuItem& MenuItem : MenuItems)
		{
			if (MenuItem.Type == TEXT("context_begin"))
			{
				MenuBuilder.BeginSection(NAME_None, FText::FromString(MenuItem.Title));
			}
			else if (MenuItem.Type == TEXT("context_end"))
			{
				MenuBuilder.EndSection();
			}
			else if (MenuItem.Type == TEXT("separator"))
			{
				MenuBuilder.AddMenuSeparator();
			}
			else
			{
				// The other menu types correspond to actual Shotgrid commands with an associated action
				FString CommandName = MenuItem.Title;
				MenuBuilder.AddMenuEntry(
					FText::FromString(CommandName),
					FText::FromString(MenuItem.Description),
					FSlateIcon(),
					FExecuteAction::CreateLambda([CommandName]()
						{
							if (UShotgridEngine* Engine = UShotgridEngine::GetInstance())
							{
								Engine->ExecuteCommand(CommandName);
							}
						})
					);
			}
		}
	}
}

bool IsEnabledWithEnvironmentVariable(FString EnvironmentVariableBase)
{
	const FString EnvBootstrapString = "UE_" + EnvironmentVariableBase + "_BOOTSTRAP";
	const FString EnvEngine = EnvironmentVariableBase + "_ENGINE";
	const FString EnvEntityType = EnvironmentVariableBase + "_ENTITY_TYPE";
	const FString EnvEntityId = EnvironmentVariableBase + "_ENTITY_ID";
	const FString EnvEnabled = "UE_" + EnvironmentVariableBase + "_ENABLED";

	bool bIsEnabled = false;
	// Check if the bootstrap environment variable is set and that the script exists
	FString ShotgridBootstrap = FPlatformMisc::GetEnvironmentVariable(*EnvBootstrapString);
	if (!ShotgridBootstrap.IsEmpty() && FPaths::FileExists(ShotgridBootstrap))
	{
		// The following environment variables must be set for the Shotgrid apps to be fully functional
		// These variables are automatically set when the editor is launched through Shotgrid Desktop
		FString ShotgridEngine = FPlatformMisc::GetEnvironmentVariable(*EnvEngine);
		FString ShotgridEntityType = FPlatformMisc::GetEnvironmentVariable(*EnvEntityType);
		FString ShotgridEntityId = FPlatformMisc::GetEnvironmentVariable(*EnvEntityId);

		if (ShotgridEngine == TEXT("tk-unreal") && !ShotgridEntityType.IsEmpty() && !ShotgridEntityId.IsEmpty())
		{
			bIsEnabled = true;

			// Set environment variable in the Python interpreter to enable the Shotgrid Unreal init script
			FString ExecCommand = FString::Printf(TEXT("import os\nos.environ['%s']='True'"), *EnvEnabled);
			IPythonScriptPlugin::Get()->ExecPythonCommand(*ExecCommand);
		}
	}
	return bIsEnabled;
}

}

struct FShotgridMenuEntryImpl
{
	FShotgridMenuEntryImpl()
	{
		UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.LevelEditorToolBar.User");
		FToolMenuSection& Section = Menu->FindOrAddSection("Shotgrid");

		const FToolMenuEntry ShotgridComboEntry = FToolMenuEntry::InitComboButton(
			TEXT("Shotgrid"),
			FUIAction(),
			FOnGetContent::CreateRaw(this, &FShotgridMenuEntryImpl::GenerateShotgridToolbarMenu),
			LOCTEXT("ShotgridCombo_Label", "Flow Production Tracking"),
			LOCTEXT("ShotgridCombo_Tooltip", "Available Flow Production commands"),
			FSlateIcon(FShotgridStyle::GetStyleSetName(), "Shotgrid.Logo"));

		Section.AddEntry(ShotgridComboEntry);

	}

	~FShotgridMenuEntryImpl()
	{
		if (!IsEngineExitRequested() && UObjectInitialized())
		{
			UToolMenus::Get()->RemoveSection("LevelEditor.LevelEditorToolBar.User", "Shotgrid");
		}
	}

	TSharedRef<SWidget> GenerateShotgridToolbarMenu()
	{
		const bool bShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

		UE::Private::Shotgrid::GenerateShotgridMenuContent(MenuBuilder, nullptr, nullptr);

		return MenuBuilder.MakeWidget();
	}

};

TUniquePtr<FShotgridUIManagerImpl> FShotgridUIManager::Instance;

class FShotgridUIManagerImpl
{
public:
	void Initialize();
	void Shutdown();

private:
	void SetupShotgridContextMenus();
	void RemoveShotgridContextMenus();

	void FillToolbar(FToolBarBuilder& ToolbarBuilder);
	TSharedRef<SWidget> GenerateShotgridToolbarMenu();

	void GenerateShotgridAssetContextMenu(FMenuBuilder& MenuBuilder, TArray<FAssetData> SelectedAssets);
	void GenerateShotgridActorContextMenu(FMenuBuilder& MenuBuilder, TArray< AActor*> SelectedActors);

	// Menu extender callbacks
	TSharedRef<FExtender> OnExtendLevelEditor(const TSharedRef<FUICommandList> CommandList, const TArray<AActor*> SelectedActors);
	TSharedRef<FExtender> OnExtendContentBrowserAssetSelectionMenu(const TArray<FAssetData>& SelectedAssets);

	FDelegateHandle ContentBrowserAssetExtenderDelegateHandle;
	FDelegateHandle LevelEditorExtenderDelegateHandle;

	bool bIsShotgridEnabled = false;

	TUniquePtr<FShotgridMenuEntryImpl> Toolbar;
};


void FShotgridUIManagerImpl::Initialize()
{
	bIsShotgridEnabled = UE::Private::Shotgrid::IsEnabledWithEnvironmentVariable(TEXT("SHOTGUN"))
		|| UE::Private::Shotgrid::IsEnabledWithEnvironmentVariable(TEXT("SHOTGRID"));

	if (bIsShotgridEnabled)
	{
		FShotgridStyle::Initialize();

		// Set the Shotgrid icons
		FShotgridStyle::SetIcon("Logo", "W18_SG_QAT_40x40");
		FShotgridStyle::SetIcon("ContextLogo", "W20_SG_QAT_24x24");

		if (!IsRunningCommandlet())
		{
			Toolbar = MakeUnique<FShotgridMenuEntryImpl>();
		}
		SetupShotgridContextMenus();
	}
}

void FShotgridUIManagerImpl::Shutdown()
{
	if (bIsShotgridEnabled)
	{
		Toolbar = {};
		RemoveShotgridContextMenus();

		FShotgridStyle::Shutdown();
	}
}

void FShotgridUIManagerImpl::SetupShotgridContextMenus()
{
	// Register Content Browser menu extender
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>(CONTENTBROWSER_MODULE_NAME);

	TArray<FContentBrowserMenuExtender_SelectedAssets>& CBAssetMenuExtenderDelegates = ContentBrowserModule.GetAllAssetViewContextMenuExtenders();
	CBAssetMenuExtenderDelegates.Add(FContentBrowserMenuExtender_SelectedAssets::CreateRaw(this, &FShotgridUIManagerImpl::OnExtendContentBrowserAssetSelectionMenu));
	ContentBrowserAssetExtenderDelegateHandle = CBAssetMenuExtenderDelegates.Last().GetHandle();

	// Register Level Editor menu extender
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(LEVELEDITOR_MODULE_NAME);

	TArray<FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors>& LevelEditorMenuExtenderDelegates = LevelEditorModule.GetAllLevelViewportContextMenuExtenders();
	LevelEditorMenuExtenderDelegates.Add(FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors::CreateRaw(this, &FShotgridUIManagerImpl::OnExtendLevelEditor));
	LevelEditorExtenderDelegateHandle = LevelEditorMenuExtenderDelegates.Last().GetHandle();
}

void FShotgridUIManagerImpl::RemoveShotgridContextMenus()
{
	if (FModuleManager::Get().IsModuleLoaded(LEVELEDITOR_MODULE_NAME))
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(LEVELEDITOR_MODULE_NAME);
		TArray<FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors>& LevelEditorMenuExtenderDelegates = LevelEditorModule.GetAllLevelViewportContextMenuExtenders();
		LevelEditorMenuExtenderDelegates.RemoveAll([this](const FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors& Delegate) { return Delegate.GetHandle() == LevelEditorExtenderDelegateHandle; });
	}

	if (FModuleManager::Get().IsModuleLoaded(CONTENTBROWSER_MODULE_NAME))
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::GetModuleChecked<FContentBrowserModule>(CONTENTBROWSER_MODULE_NAME);
		TArray<FContentBrowserMenuExtender_SelectedAssets>& CBAssetMenuExtenderDelegates = ContentBrowserModule.GetAllAssetViewContextMenuExtenders();
		CBAssetMenuExtenderDelegates.RemoveAll([this](const FContentBrowserMenuExtender_SelectedAssets& Delegate) { return Delegate.GetHandle() == ContentBrowserAssetExtenderDelegateHandle; });
	}
}

void FShotgridUIManagerImpl::GenerateShotgridAssetContextMenu(FMenuBuilder& MenuBuilder, TArray<FAssetData> SelectedAssets)
{
	UE::Private::Shotgrid::GenerateShotgridMenuContent(MenuBuilder, &SelectedAssets, nullptr);
}

void FShotgridUIManagerImpl::GenerateShotgridActorContextMenu(FMenuBuilder& MenuBuilder, TArray<AActor*> SelectedActors)
{
	UE::Private::Shotgrid::GenerateShotgridMenuContent(MenuBuilder, nullptr, &SelectedActors);
}

TSharedRef<FExtender> FShotgridUIManagerImpl::OnExtendContentBrowserAssetSelectionMenu(const TArray<FAssetData>& SelectedAssets)
{
	// Menu extender for Content Browser context menu when an asset is selected
	TSharedRef<FExtender> Extender(new FExtender());

	if (SelectedAssets.Num() > 0)
	{
		Extender->AddMenuExtension("AssetContextReferences", EExtensionHook::After, nullptr, FMenuExtensionDelegate::CreateLambda(
			[this, SelectedAssets](FMenuBuilder& MenuBuilder)
			{
				MenuBuilder.AddMenuSeparator();
				MenuBuilder.AddSubMenu(
					LOCTEXT("Shotgrid_ContextMenu", "ShotGrid"),
					FText(),
					FNewMenuDelegate::CreateRaw(this, &FShotgridUIManagerImpl::GenerateShotgridAssetContextMenu, SelectedAssets),
					false,
					FSlateIcon(FShotgridStyle::GetStyleSetName(), "Shotgrid.ContextLogo")
				);
			}));
	}

	return Extender;
}

TSharedRef<FExtender> FShotgridUIManagerImpl::OnExtendLevelEditor(const TSharedRef<FUICommandList> CommandList, const TArray<AActor*> SelectedActors)
{
	// Menu extender for Level Editor and World Outliner context menus when an actor is selected
	TSharedRef<FExtender> Extender(new FExtender());

	if (SelectedActors.Num() > 0)
	{
		Extender->AddMenuExtension("ActorUETools", EExtensionHook::After, nullptr, FMenuExtensionDelegate::CreateLambda(
			[this, SelectedActors](FMenuBuilder& MenuBuilder)
			{
				MenuBuilder.AddSubMenu(
					LOCTEXT("Shotgrid_ContextMenu", "ShotGrid"),
					FText(),
					FNewMenuDelegate::CreateRaw(this, &FShotgridUIManagerImpl::GenerateShotgridActorContextMenu, SelectedActors),
					false,
					FSlateIcon(FShotgridStyle::GetStyleSetName(), "Shotgrid.ContextLogo")
				);
			}));
	}

	return Extender;
}

void FShotgridUIManager::Initialize()
{
	if (!Instance.IsValid())
	{
		Instance = MakeUnique<FShotgridUIManagerImpl>();
		Instance->Initialize();
	}
}

void FShotgridUIManager::Shutdown()
{
	if (Instance.IsValid())
	{
		Instance->Shutdown();
		Instance.Reset();
	}
}

#undef LOCTEXT_NAMESPACE
