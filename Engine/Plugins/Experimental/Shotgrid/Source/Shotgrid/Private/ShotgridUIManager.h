// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UniquePtr.h"

class FShotgridUIManagerImpl;

class FShotgridUIManager
{
public:
	static void Initialize();
	static void Shutdown();

private:
	static TUniquePtr<FShotgridUIManagerImpl> Instance;
};
