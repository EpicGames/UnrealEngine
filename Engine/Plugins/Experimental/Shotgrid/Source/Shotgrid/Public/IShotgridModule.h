// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"		// For inline LoadModuleChecked()

#define SHOTGUN_MODULE_NAME TEXT("Shotgrid")

/**
 * The public interface of the Shotgrid module
 */
class IShotgridModule : public IModuleInterface
{

public:

	/**
	 * Singleton-like access to IShotgridPlugin
	 *
	 * @return Returns Shotplugin singleton instance, loading the module on demand if needed
	 */
	static inline IShotgridModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IShotgridModule>(SHOTGUN_MODULE_NAME);
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(SHOTGUN_MODULE_NAME);
	}
};

