// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class Shotgrid : ModuleRules
	{
		public Shotgrid(ReadOnlyTargetRules Target)
			: base(Target)
		{
			PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetRegistry",
					"ContentBrowser",
					"Core",
					"CoreUObject",
					"EditorStyle",
					"Engine",
					"LevelEditor",
					"PythonScriptPlugin",
					"Settings",
					"Slate",
					"SlateCore",
					"ToolMenus",
					"UnrealEd"
				}
			);
		}
	}
}
